Portions of this plugin, Copyright 2019 Avid Technology, Inc.