This plugin uses GPU hardware accelerated video decoding on Windows via Direct3D 12 Video.
See: https://learn.microsoft.com/en-us/windows/win32/medfound/direct3d-12-video-overview

At present this plugin supports the following codecs, provided there is support by your GPU
(via vendor provided extensions) with some limitations:

H.264 / AVC
- Only Baseline, Main and High profiles are supported
- Constrained Baseline profile can be used (no FMO, ASO and RS)
- PAFF and MBAFF are not supported (interlaced video)


H.265 / HEVC
- Only Main and Main10 profiles up to level 6.3 are supported




NOTE on usage:
==============

By default, even with this plugin enabled, its use is disabled on Windows due to the
vast combinations of GPUs and driver implementations.
It can be enabled via CVar / .ini file
To enable via ini file add
	[ConsoleVariables]
	ElectraDecoders.bDoNotUseD3D12Video=false
to the respective platform's ini file.

To enable by default at compile time, override the method
		protected virtual bool bIsDefaultIgnoredOnPlatform
		{
			get
			{
				return false;
			}
		}
from the base D3D12VideoDecodersElectra.Build.cs build script in the platform build
scripts where you want it enabled.

For testing you should confirm operation by setting the console variable
	`ElectraDecoders.bDoNotUseD3D12Video`
to `false`.



To disable the plugin right at Startup() you can use the CVar / ini setting:
	[ConsoleVariables]
	ElectraDecoders.bDisableD3D12Video=true

This acts as a master switch. When disabled `bDoNotUseD3D12Video` has no effect.
