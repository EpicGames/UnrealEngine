# RTSP Media

An experimental plugin for Unreal Engine's Media Framework containing a cross-platform RTSP media source implementation with H.264 support.

Utilises ElectraDecoders for H.264 decoding. 

## Platforms

- Windows
- macOS
- Linux

## Features

- TCP interleaved transport to ensure packet delivery.
- Configurable jitter buffer to smooth network delivery spikes. Set this to zero to for minimum latency on clean networks.
- Automatic reconnection on connection failure.
- Configurable decoder buffer size. Reduce to 1 for streams that don't contain H.264 B-frames to minimise latency.

## Linux Setup

Linux support requires an installation of libavcodec and libavutil to be available on the local system. 

Instructions to set this up are available in the following location:

`Engine/Source/ThirdParty/libav/README`

## Current Limitations

- H.264 only
- TCP interleaved transport only
  - No UDP support
- Fixed configuration per session
  - No automatic jitter buffer adjustment
- No audio support
- No authentication support
